// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class CineCameraSceneCapture : ModuleRules
{
	public CineCameraSceneCapture(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"CoreUObject",
				"Engine",
				"Slate",
				"SlateCore",
				"CinematicCamera",
				"RenderCore",
				"Renderer",
			}
		);

		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"OpenColorIO",
			}
		);
	}
}
