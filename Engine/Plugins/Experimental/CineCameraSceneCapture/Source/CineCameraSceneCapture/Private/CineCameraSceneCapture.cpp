// Copyright Epic Games, Inc. All Rights Reserved.

#include "CineCameraSceneCapture.h"

#define LOCTEXT_NAMESPACE "FCineCameraSceneCaptureModule"

void FCineCameraSceneCaptureModule::StartupModule()
{
}

void FCineCameraSceneCaptureModule::ShutdownModule()
{
}

#undef LOCTEXT_NAMESPACE
	
IMPLEMENT_MODULE(FCineCameraSceneCaptureModule, CineCameraSceneCapture)