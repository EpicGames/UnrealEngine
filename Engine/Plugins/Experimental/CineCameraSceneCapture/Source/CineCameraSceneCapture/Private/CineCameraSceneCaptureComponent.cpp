// Copyright Epic Games, Inc. All Rights Reserved.

#include "CineCameraSceneCaptureComponent.h"
#include "Components/StaticMeshComponent.h"
#include "SceneViewExtension.h"
#include "Runtime/CinematicCamera/Public/CineCameraComponent.h"
#include "Runtime/Engine/Public/SceneView.h"
#include "Engine/TextureRenderTarget2D.h"
#include "Framework/Notifications/NotificationManager.h"
#include "OpenColorIODisplayExtension.h"
#include "OpenColorIORendering.h"
#include "PostProcess/PostProcessMaterialInputs.h"
#include "ScreenPass.h"
#include "Widgets/Notifications/SNotificationList.h"

#define LOCTEXT_NAMESPACE "FCineCameraSceneCaptureModule"

DEFINE_LOG_CATEGORY(LogCineCapture);

#define CINE_CAMERA_INVALID_PARENT_WARNING "Cine Capture requires to be parented to Cine Camera Component. Cine Capture {0} on Actor \"{1}\" will be disabled until it is parented to Cine Camera Actor."
#define CINE_CAMERA_INVALID_PARENT_WARNING_FMT "Cine Capture requires to be parented to Cine Camera Component. Cine Capture %s on Actor \"%s\" will be disabled until it is parented to Cine Camera Actor."

// This extension is only registered onto the scene capture 2d component, and therefore runs locally.
class FCineCameraCaptureSceneViewExtension : public ISceneViewExtension, public TSharedFromThis<FCineCameraCaptureSceneViewExtension, ESPMode::ThreadSafe>
{
public:

	/** Constructor */
	FCineCameraCaptureSceneViewExtension(TWeakObjectPtr<UCineCaptureComponent2D> InCineCaptureComponent)
		: CineCaptureComponentWeak(InCineCaptureComponent)
		, DeltaTime(0.0f)
		, CachedResourcesRenderThread()
		, bIsOcioEnabledRenderThread(false)
	{ }

	virtual ~FCineCameraCaptureSceneViewExtension() = default;

	//~ Begin FSceneViewExtensionBase Interface
	virtual void SetupViewFamily(FSceneViewFamily& InViewFamily) override {};
	virtual void SetupView(FSceneViewFamily& InViewFamily, FSceneView& InView) override
	{
		TStrongObjectPtr<UCineCaptureComponent2D> CineCaptureComponent = CineCaptureComponentWeak.Pin();

		if (!CineCaptureComponent.IsValid())
		{
			return;
		}

		UCineCameraComponent* CineCameraComponent = CineCaptureComponent->CineCameraComponent.Get();
		if (!IsValid(CineCameraComponent))
		{
			return;
		}

		FMinimalViewInfo DesiredView;

		// Immitate the behaviour of viewports.
		CineCameraComponent->GetCameraView(DeltaTime, DesiredView);
		const FVector ViewLocation = CineCameraComponent->GetComponentToWorld().GetTranslation();
		InView.StartFinalPostprocessSettings(ViewLocation);
		if (CineCaptureComponent->bOverrideUserFlags)
		{
			DesiredView.PostProcessSettings.bOverride_UserFlags = CineCaptureComponent->bOverrideUserFlags;
			DesiredView.PostProcessSettings.UserFlags = CineCaptureComponent->UserFlags;
		}
		InView.OverridePostProcessSettings(DesiredView.PostProcessSettings, 1.0);
		FSceneViewInitOptions ViewInitOptions;
		InView.EndFinalPostprocessSettings(ViewInitOptions);

		// Required for certain effects (lighting) to match that of Cine Camera.
		InView.bIsSceneCapture = CineCaptureComponent->bFollowSceneCaptureRenderPath;

		bIsOcioEnabledRenderThread = CineCaptureComponent->OCIOConfiguration.bIsEnabled;

		// Setup OCIO.
		if (bIsOcioEnabledRenderThread)
		{
			FOpenColorIORenderPassResources PassResources = FOpenColorIORendering::GetRenderPassResources(CineCaptureComponent->OCIOConfiguration.ColorConfiguration, InViewFamily.GetFeatureLevel());

			if (PassResources.IsValid())
			{
				FOpenColorIORendering::PrepareView(InViewFamily, InView);
			}

			ENQUEUE_RENDER_COMMAND(ProcessColorSpaceTransform)(
				[WeakThis = AsWeak(), ResourcesRenderThread = MoveTemp(PassResources)](FRHICommandListImmediate& RHICmdList)
				{
					TSharedPtr<FCineCameraCaptureSceneViewExtension> This = WeakThis.Pin();
					if (This.IsValid())
					{
						//Caches render thread resource to be used when applying configuration in PostRenderViewFamily_RenderThread
						This->CachedResourcesRenderThread = ResourcesRenderThread;
					}
				}
			);
		}
	}

	//~ End FSceneViewExtensionBase Interface

	/** 
	* Delta time is needed for the purposes of camera smoothing.
	*/
	void SetFrameDeltaTime(float InDeltaTime)
	{
		DeltaTime = InDeltaTime;
	}

	virtual void SubscribeToPostProcessingPass(EPostProcessingPass PassId, const FSceneView& View, FAfterPassCallbackDelegateArray& InOutPassCallbacks, bool bIsPassEnabled)
	{
		if (!bIsOcioEnabledRenderThread)
		{
			return;
		}

		if (PassId == EPostProcessingPass::Tonemap)
		{
			InOutPassCallbacks.Add(FAfterPassCallbackDelegate::CreateRaw(this, &FCineCameraCaptureSceneViewExtension::PostProcessPassAfterTonemap_RenderThread));
		}
	}

	FScreenPassTexture PostProcessPassAfterTonemap_RenderThread(FRDGBuilder& GraphBuilder, const FSceneView& View, const FPostProcessMaterialInputs& InOutInputs)
	{
		const FScreenPassTexture& SceneColor = FScreenPassTexture::CopyFromSlice(GraphBuilder, InOutInputs.GetInput(EPostProcessMaterialInput::SceneColor));
		check(SceneColor.IsValid());

		FScreenPassRenderTarget Output = InOutInputs.OverrideOutput;

		// If the override output is provided, it means that this is the last pass in post processing.
		if (!Output.IsValid())
		{
			Output = FScreenPassRenderTarget::CreateFromInput(GraphBuilder, SceneColor, View.GetOverwriteLoadAction(), TEXT("OCIORenderTarget"));
		}

		FOpenColorIORendering::AddPass_RenderThread(
			GraphBuilder,
			View,
			SceneColor,
			Output,
			CachedResourcesRenderThread
		);

		return MoveTemp(Output);
	}

private:
	/**
	* A transient property that is used to deliver settings from CineCaptureComponent to views that are related to cine capture.
	*/
	TWeakObjectPtr<UCineCaptureComponent2D> CineCaptureComponentWeak;

	/**
	* Delta time between frames. Used for camera smoothing.
	*/
	float DeltaTime;

	/** Cached pass resources required to apply conversion for render thread */
	FOpenColorIORenderPassResources CachedResourcesRenderThread;

	/** Indicates if OCIO is enabled. */
	std::atomic<bool> bIsOcioEnabledRenderThread;
};


UCineCaptureComponent2D::UCineCaptureComponent2D(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer), RenderTargetHighestDimension(1280), bFollowSceneCaptureRenderPath(true)
{
	CaptureSource = ESceneCaptureSource::SCS_FinalToneCurveHDR;
	bAlwaysPersistRenderingState = true;

	// Enable HWRT by default on the scene capture render path to minimize artifacts
	bUseRayTracingIfEnabled = true;
}

void UCineCaptureComponent2D::UpdateSceneCaptureContents(FSceneInterface* Scene, ISceneRenderBuilder& SceneRenderBuilder)
{
	if (!CineCameraComponent.IsValid() || !CineCaptureSVE.IsValid())
	{
		return;
	}

	SetRelativeTransform(FTransform::Identity);
	
	PostProcessSettings = CineCameraComponent->PostProcessSettings;
	FOVAngle = CineCameraComponent->FieldOfView;
	FirstPersonFieldOfView = CineCameraComponent->FirstPersonFieldOfView;
	FirstPersonScale = CineCameraComponent->FirstPersonScale;
	bOverride_CustomNearClippingPlane = CineCameraComponent->bOverride_CustomNearClippingPlane;
	CustomNearClippingPlane = CineCameraComponent->CustomNearClippingPlane;
	bEnableFirstPersonFieldOfView = CineCameraComponent->bEnableFirstPersonFieldOfView;
	bEnableFirstPersonScale = CineCameraComponent->bEnableFirstPersonScale;

	Scene->UpdateSceneCaptureContents(this, SceneRenderBuilder);
}

void UCineCaptureComponent2D::CheckResizeRenderTarget()
{
	// Making sure that censor's aspect ratio is reflected render target and that RenderTarget Dimension specified by the user is taken into account.
	if (TextureTarget)
	{
		float Ratio = TextureTarget->GetSurfaceWidth() / TextureTarget->GetSurfaceHeight();
		const float SmallNum = 1e-3f;
		if ((FMath::Abs(Ratio - CineCameraComponent->AspectRatio) > SmallNum)
			|| (FMath::Max(TextureTarget->GetSurfaceWidth(), TextureTarget->GetSurfaceHeight()) != RenderTargetHighestDimension))
		{
			if (CineCameraComponent->AspectRatio >= 1.)
			{
				int32 LowestDim = FMath::Max(RenderTargetHighestDimension / CineCameraComponent->AspectRatio, 1);
				TextureTarget->ResizeTarget(RenderTargetHighestDimension, LowestDim);
			}
			else
			{
				int32 LowestDim = FMath::Max(RenderTargetHighestDimension * CineCameraComponent->AspectRatio, 1);
				TextureTarget->ResizeTarget(LowestDim, RenderTargetHighestDimension);
			}
		}
	}
}

void UCineCaptureComponent2D::OnRegister()
{
	Super::OnRegister();

#if WITH_EDITORONLY_DATA
	// Remove mesh created by Scene Capture Component.
	if (ProxyMeshComponent)
	{
		ProxyMeshComponent->DestroyComponent();
		ProxyMeshComponent = nullptr;
	}
#endif

	if (!CineCaptureSVE.IsValid())
	{
		// This will create a new scene extension per each capture.
		CineCaptureSVE = MakeShared<FCineCameraCaptureSceneViewExtension>(this);
		SceneViewExtensions.Add(CineCaptureSVE);
	}

	ValidateCineCameraComponent();
}

void UCineCaptureComponent2D::OnUnregister()
{
	Super::OnUnregister();

	if (CineCaptureSVE.IsValid())
	{
		SceneViewExtensions.Remove(CineCaptureSVE);
		CineCaptureSVE.Reset();
	}

	CineCameraComponent = nullptr;
}

void UCineCaptureComponent2D::OnAttachmentChanged()
{
	// If no parent is present, then this component is in a transient state.
	if (USceneComponent* ParentComponent = GetAttachParent())
	{
		ValidateCineCameraComponent();
	}
}

void UCineCaptureComponent2D::TickComponent(float DeltaTime, ELevelTick TickType, FActorComponentTickFunction* ThisTickFunction)
{
	Super::TickComponent(DeltaTime, TickType, ThisTickFunction);

	if (CineCaptureSVE.IsValid())
	{
		CineCaptureSVE->SetFrameDeltaTime(DeltaTime);
	}

	// Since CineCameraComponent properties can change any time, render target needs to be validated every frame.
	// The rest of the properties are copied right before the view is setup.
	if (CineCameraComponent.IsValid())
	{
		CheckResizeRenderTarget();
	}
}

void UCineCaptureComponent2D::ValidateCineCameraComponent()
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (!IsThisNotNull(this, "UCineCaptureComponent2D::ValidateCineCameraComponent") || !IsValidChecked(this))
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	{
		return;
	}

	CineCameraComponent = Cast<UCineCameraComponent>(GetAttachParent());

	if (!CineCameraComponent)
	{
#if WITH_EDITOR
		FNotificationInfo Info(FText::Format(LOCTEXT("AddCineCameraNotification", CINE_CAMERA_INVALID_PARENT_WARNING), FText::FromString(GetName()), FText::FromString(GetOuter()->GetName())));
		Info.ExpireDuration = 5.0f;

		FSlateNotificationManager::Get().AddNotification(Info);
#endif
		UE_LOG(LogCineCapture, Warning, TEXT(CINE_CAMERA_INVALID_PARENT_WARNING_FMT), *GetName(), *GetOuter()->GetName());
	}
}


#undef LOCTEXT_NAMESPACE
