// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OpenColorIOColorSpace.h"
#include "Components/SceneCaptureComponent2D.h"
#include "CineCameraSceneCaptureComponent.generated.h"

DECLARE_LOG_CATEGORY_EXTERN(LogCineCapture, Log, All);

class FCineCameraCaptureSceneViewExtension;
class UCineCameraComponent;

/**
* Cine Capture Component extends Scene Capture to allow users to render Cine Camera Component into a render target. 
* Cine Capture has a few modifiable properties, but most of the properties are controlled by Cine Camera Component.
* Cine Capture Component is required to be parented to Cine Camera Component or a class that extends it.
* 
*/
UCLASS(hidecategories = (Transform, Collision, Object, Physics, SceneComponent, PostProcessVolume, Projection, Rendering, PlanarReflection), ClassGroup = Rendering, editinlinenew, meta = (BlueprintSpawnableComponent))
class CINECAMERASCENECAPTURE_API UCineCaptureComponent2D : public USceneCaptureComponent2D
{
	GENERATED_UCLASS_BODY()
public:
	
	/** 
	* Highest possible dimension of specified render target in either X or Y (depends on cine camera aspect ratio). Used to adjust the render target size.
	* Example: With cine camera's censor aspect ratio is 2:1, and Highest Dimension set to 1000, the render target will be resized to 1000x500. 
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Capture Settings", meta = (ClampMin = "1", ClampMax = "8192", UIMin = "1", UIMax = "8192"))
	int32 RenderTargetHighestDimension;

	/** 
	* Affects rendering path cine capture takes. Scene Capture takes a slightly different rendering route compared to viewport rendering
	* for the purposes of optimization. If the results of cine capture are vastly different to what cine camera displays try disabling this.
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Capture Settings", AdvancedDisplay)
	bool bFollowSceneCaptureRenderPath;

	/** Override state for per-view user flags accessible in materials via TestPostVolumeUserFlag node. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Capture Settings", AdvancedDisplay)
	bool bOverrideUserFlags;

	/** Per-view user flags accessible in materials via TestPostVolumeUserFlag node, allowing per-view overrides of material behavior. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Capture Settings", AdvancedDisplay, meta = (EditCondition = "bOverrideUserFlags"))
	int32 UserFlags;

	/** 
	* Open Color IO Configuration. Used to apply color transforms.
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Color", meta = (DisplayName = "OpenColorIO Display Configuration"))
	FOpenColorIODisplayConfiguration OCIOConfiguration;

	/** Used to gather relevant properties for View rendering. */
	UPROPERTY()
	TSoftObjectPtr<UCineCameraComponent> CineCameraComponent;

public:
	virtual void UpdateSceneCaptureContents(FSceneInterface* Scene, ISceneRenderBuilder& SceneRenderBuilder) override;
	virtual void OnRegister() override;
	virtual void OnUnregister() override;

	/** Used for validation of this componet's attachment to Cine Camera Component. */
	virtual void OnAttachmentChanged() override; 
	virtual void TickComponent(float DeltaTime, enum ELevelTick TickType, FActorComponentTickFunction* ThisTickFunction) override;

private:
	void CheckResizeRenderTarget();
	void ValidateCineCameraComponent();

private:
	/** This scene view extension is used to get ahold of views during the setup process. */
	TSharedPtr<FCineCameraCaptureSceneViewExtension, ESPMode::ThreadSafe> CineCaptureSVE;
};
